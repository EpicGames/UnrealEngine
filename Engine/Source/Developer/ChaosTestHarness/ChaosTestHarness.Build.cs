// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ChaosTestHarness : TestModuleRules
{
	public ChaosTestHarness(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(new string[] {
			"Catch2"
		});

		PrivateDependencyModuleNames.AddRange(new string[] {
			"Core",
			"CoreUObject",
			"Engine",
			"LowLevelTestsRunner",
		});

		// Temporarily link to base Chaos modules - remove once everything is moved into the new tree
		SetupModulePhysicsSupport(Target);

		bWarningsAsErrors = true;
	}
}