// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.All)]
public class ChaosTestHarnessTarget : TestTargetRules
{
	public ChaosTestHarnessTarget(TargetInfo Target) : base(Target)
	{
		bCompileAgainstEngine = true;
		bCompileAgainstCoreUObject = true;
		bCompileAgainstApplicationCore = true;
		bMockEngineDefaults = true;
		bUsePlatformFileStub = true;

		bWithLowLevelTestsOverride = true;

		GlobalDefinitions.Add("CATCH_CONFIG_ENABLE_BENCHMARKING=1");

		bWarningsAsErrors = true;
	}
}