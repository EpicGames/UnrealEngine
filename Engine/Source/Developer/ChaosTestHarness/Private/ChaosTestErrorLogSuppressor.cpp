// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/LowLevelTest/ChaosTestErrorLogSuppressor.h"

#include "Misc/OutputDeviceRedirector.h"

namespace Chaos::LowLevelTest
{
	FErrorLogSuppressor::FErrorLogSuppressor()
		: OldWarn(GWarn)
	{
		GWarn = this;
		GLog->AddOutputDevice(this);
	}

	FErrorLogSuppressor::~FErrorLogSuppressor()
	{
		GLog->RemoveOutputDevice(this);
		GWarn = OldWarn;
	}

	FErrorLogSuppressor& FErrorLogSuppressor::ExpectError(FStringView InPattern)
	{
		ErrorPatterns.Emplace(InPattern);
		return *this;
	}

	FErrorLogSuppressor& FErrorLogSuppressor::ExpectWarning(FStringView InPattern)
	{
		WarningPatterns.Emplace(InPattern);
		return *this;
	}

	void FErrorLogSuppressor::Serialize(const TCHAR* InData, ELogVerbosity::Type InVerbosity, const FName& InCategory)
	{
		if (InVerbosity == ELogVerbosity::Error)
		{
			FStringView Message(InData);
			for (const FString& Pattern : ErrorPatterns)
			{
				// If the message matches our pattern, swallow the error and continue
				if (Message.Contains(Pattern))
				{
					return;
				}
			}
		}
		else if (InVerbosity == ELogVerbosity::Warning)
		{
			FStringView Message(InData);
			for (const FString& Pattern : WarningPatterns)
			{
				if (Message.Contains(Pattern))
				{
					return;
				}
			}
		}
		// Otherwise, forward as normal
		if (OldWarn)
		{
			OldWarn->Serialize(InData, InVerbosity, InCategory);
		}
	}
} // namespace Chaos::LowLevelTest
