// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/LowLevelTest/ChaosTestScene.h"
#include "Chaos/Box.h"
#include "Chaos/MassProperties.h"
#include "Chaos/Sphere.h"
#include "ChaosSolversModule.h"
#include "EventsData.h"
#include "PBDRigidsSolver.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "Chaos/ChaosScene.h"


namespace Chaos::LowLevelTest
{

	FChaosTestScene::FChaosTestScene()
		: NumTicks(0)
	{
		CreateScene();
		CreateSolver();
	}

	FChaosTestScene::~FChaosTestScene()
	{
		DestroySolver();
		DestroyScene();
	}

	Chaos::FPBDRigidsSolver* FChaosTestScene::GetSolver()
	{
		return Solver;
	}

	void FChaosTestScene::TickSolver(float Dt)
	{
		Solver->AdvanceAndDispatch_External(Dt);
		Solver->UpdateGameThreadStructures();
		Solver->SyncEvents_GameThread();
		++NumTicks;
	}

	bool FChaosTestScene::TickSolverToSleep(float Dt, int32 MaxTicks)
	{
		for (int32 SleepTickIndex = 0; (SleepTickIndex < MaxTicks) && !IsSceneSleeping(); ++SleepTickIndex)
		{
			TickSolver(Dt);
		}

		return IsSceneSleeping();
	}

	bool FChaosTestScene::IsSceneSleeping() const
	{
		using namespace Chaos;

		for (Chaos::FSingleParticlePhysicsProxy* Proxy : Proxies)
		{
			if (Proxy->GetGameThreadAPI().ObjectState() == EObjectStateType::Dynamic)
			{
				return false;
			}
		}

		return true;
	}

	int FChaosTestScene::GetNumTicks() const
	{
		return NumTicks;
	}

	Chaos::FSingleParticlePhysicsProxy* FChaosTestScene::AddDynamicSphere(const TCHAR* InName, const FTransform3d& InTransform, const double& InRadius)
	{
		using namespace Chaos;

		FSingleParticlePhysicsProxy* Proxy = CreateDynamicSphere(InName, InTransform, InRadius);

		Solver->RegisterObject(Proxy);

		return Proxy;
	}

	Chaos::FSingleParticlePhysicsProxy* FChaosTestScene::AddDynamicBox(const TCHAR* InName, const FTransform3d& InTransform, const FVector3d& InHalfExtents)
	{
		using namespace Chaos;

		FSingleParticlePhysicsProxy* Proxy = CreateDynamicBox(InName, InTransform, InHalfExtents);

		Solver->RegisterObject(Proxy);

		return Proxy;
	}

	Chaos::FSingleParticlePhysicsProxy* FChaosTestScene::AddDynamicProxy(const TCHAR* InName, const FTransform3d& InTransform, const Chaos::FImplicitObjectPtr& InGeometry)
	{
		using namespace Chaos;

		FSingleParticlePhysicsProxy* Proxy = CreateDynamicProxy(InName, InTransform, InGeometry);

		Solver->RegisterObject(Proxy);

		return Proxy;
	}

	Chaos::FSingleParticlePhysicsProxy* FChaosTestScene::AddKinematicBox(const TCHAR* InName, const FTransform3d& InTransform, const FVector3d& InHalfExtents)
	{
		using namespace Chaos;

		FSingleParticlePhysicsProxy* Proxy = CreateDynamicBox(InName, InTransform, InHalfExtents);
		Proxy->GetGameThreadAPI().SetObjectState(EObjectStateType::Kinematic);

		Solver->RegisterObject(Proxy);

		return Proxy;
	}

	Chaos::FSingleParticlePhysicsProxy* FChaosTestScene::AddKinematicProxy(const TCHAR* InName, const FTransform3d& InTransform, const Chaos::FImplicitObjectPtr& InGeometry)
	{
		using namespace Chaos;

		FSingleParticlePhysicsProxy* Proxy = CreateDynamicProxy(InName, InTransform, InGeometry);
		Proxy->GetGameThreadAPI().SetObjectState(EObjectStateType::Kinematic);

		Solver->RegisterObject(Proxy);

		return Proxy;
	}

	void FChaosTestScene::SetGravity(const FVector3d& InG)
	{
		Solver->GetEvolution()->GetGravityForces().SetAcceleration(InG, 0);
	}


	void FChaosTestScene::SetCollisionCallback(const TFunction<void(const Chaos::FCollisionEventData&)>& InCollisionCallback)
	{
		CollisionCallback = InCollisionCallback;
	}

	void FChaosTestScene::CreateScene()
	{
		float AsyncDt = -1.0f;
		Scene = new FChaosScene(nullptr, AsyncDt);
	}

	void FChaosTestScene::DestroyScene()
	{
		delete Scene;
		Scene = nullptr;
	}

	void FChaosTestScene::CreateSolver()
	{
		using namespace Chaos;

		const float AsyncDt = -1.0f;	// No async
		Solver = FChaosSolversModule::GetModule()->CreateSolver(nullptr, AsyncDt, Chaos::EThreadingMode::SingleThread, TEXT("Collision Test Scene"));
		Solver->PhysSceneHack = Scene;

		Chaos::FEventManager* EventManager = Solver->GetEventManager();
		EventManager->RegisterHandler<Chaos::FCollisionEventData>(Chaos::EEventType::Collision, this, &FChaosTestScene::OnCollision);
	}

	void FChaosTestScene::DestroySolver()
	{
		using namespace Chaos;

		FChaosSolversModule::GetModule()->DestroySolver(Solver);
		Solver = nullptr;
	}

	Chaos::FSingleParticlePhysicsProxy* FChaosTestScene::CreateDynamicSphere(const TCHAR* InName, const FTransform3d& InTransform, const double& InRadius)
	{
		using namespace Chaos;
		FImplicitObjectPtr SphereGeom = FImplicitObjectPtr(new Chaos::FImplicitSphere3(FVector3f::Zero(), (float)InRadius));
		return CreateDynamicProxy(InName, InTransform, SphereGeom);
	}

	Chaos::FSingleParticlePhysicsProxy* FChaosTestScene::CreateDynamicBox(const TCHAR* InName, const FTransform3d& InTransform, const FVector3d& InHalfExtents)
	{
		using namespace Chaos;
		FVector3d HalfExtents = InHalfExtents * InTransform.GetScale3D().GetAbs();
		FImplicitObjectPtr BoxGeom = FImplicitObjectPtr(new Chaos::FImplicitBox3(-HalfExtents, HalfExtents));
		return CreateDynamicProxy(InName, InTransform, BoxGeom);
	}

	Chaos::FSingleParticlePhysicsProxy* FChaosTestScene::CreateDynamicProxy(const TCHAR* InName, const FTransform3d& InTransform, const Chaos::FImplicitObjectPtr& InGeometry)
	{
		using namespace Chaos;

		FSingleParticlePhysicsProxy* Proxy = FSingleParticlePhysicsProxy::Create(Chaos::TPBDRigidParticle<FReal, 3>::CreateParticle());
		FRigidBodyHandle_External& ParticleGT = Proxy->GetGameThreadAPI();

	#if CHAOS_DEBUG_NAME
		if (InName != nullptr)
		{
			TSharedPtr<FString> NamePtr = MakeShared<FString>(InName);
			ParticleGT.SetDebugName(NamePtr);
		}
	#endif
		ParticleGT.SetGeometry(InGeometry);
		ParticleGT.SetX(InTransform.GetLocation());
		ParticleGT.SetR(InTransform.GetRotation());

		InitCollisionFilter(Proxy);
		InitMassProperties(Proxy);

		Proxies.Add(Proxy);

		return Proxy;
	}

	void FChaosTestScene::InitCollisionFilter(Chaos::FSingleParticlePhysicsProxy* Proxy)
	{
		using namespace Chaos;

		constexpr EFilterFlags FilterFlags = EFilterFlags::SimpleCollision | EFilterFlags::ContactNotify | EFilterFlags::ModifyContacts;
		const Filter::FShapeFilterData ShapeFilter = Filter::FShapeFilterBuilder::BuildBlockAll(FilterFlags);

		for (int32 ShapeIndex = 0; ShapeIndex < Proxy->GetGameThreadAPI().ShapesArray().Num(); ++ShapeIndex)
		{
			Proxy->GetGameThreadAPI().SetShapeSimCollisionEnabled(ShapeIndex, true);
			Proxy->GetGameThreadAPI().SetShapeQueryCollisionEnabled(ShapeIndex, true);
			Proxy->GetGameThreadAPI().SetShapeFilterData(ShapeIndex, ShapeFilter);
		}
	}

	void FChaosTestScene::InitMassProperties(Chaos::FSingleParticlePhysicsProxy* Proxy)
	{
		using namespace Chaos;

		FRigidBodyHandle_External& ParticleGT = Proxy->GetGameThreadAPI();
		const FShapesArray& ShapesArray = ParticleGT.ShapesArray();


		float DensityKgPerCm3 = 1.e-3f;	// Water 1g/cm3

		// Calculate mass and inertia
		FMassProperties MassProps;
		Chaos::CalculateMassPropertiesFromShapeCollection(
			MassProps,
			ShapesArray.Num(),
			DensityKgPerCm3,
			TArray<bool>(),
			[&ShapesArray](int32 ShapeIndex) { return ShapesArray[ShapeIndex].Get(); }
		);

		// Diagonalize the inertia
		TransformToLocalSpace(MassProps);

		// Set up particle mass
		if (MassProps.Mass > 0)
		{
			ParticleGT.SetM(MassProps.Mass);
			ParticleGT.SetInvM(1.0f / MassProps.Mass);
		}
		else
		{
			ParticleGT.SetM(0);
			ParticleGT.SetInvM(0);
		}
		if (MassProps.InertiaTensor.GetDiagonal().GetAbsMin() > 0)
		{
			ParticleGT.SetI(MassProps.InertiaTensor.GetDiagonal());
			ParticleGT.SetInvI(FVec3(1.0f) / MassProps.InertiaTensor.GetDiagonal());
		}
		else
		{
			ParticleGT.SetI(FVec3(0));
			ParticleGT.SetInvI(FVec3(0));
		}
		ParticleGT.SetCenterOfMass(MassProps.CenterOfMass);
		ParticleGT.SetRotationOfMass(MassProps.RotationOfMass);
	}

	void FChaosTestScene::OnCollision(const Chaos::FCollisionEventData& CollisionEventData)
	{
		if (CollisionCallback.IsSet())
		{
			CollisionCallback(CollisionEventData);
		}
	}

} // Chaos::LowLevelTest