// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Chaos/LowLevelTest/ChaosTestHarness.h"

#include "CompGeom/ExactPredicates.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "TestCommon/Initialization.h"

// Pull in the log category declarations for below overrides
#include "SlateGlobals.h"
#include "Styling/SlateWidgetStyleContainerBase.h"

#include "catch2/catch_tag_alias_autoregistrar.hpp"

CATCH_REGISTER_TAG_ALIAS("[@ChaosTests]", "[Chaos]~[!benchmark]")
CATCH_REGISTER_TAG_ALIAS("[@ChaosBenchmarks]", "[Chaos][!benchmark]")

GROUP_BEFORE_GLOBAL(Catch::DefaultGroup)
{
	// Find the engine directory before InitAll() enables the platform file stub. Prevents warnings.
	FPlatformMisc::EngineDir();
	UE::Geometry::ExactPredicates::GlobalInit();

	// Temporarily disable warnings during init
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogSlate, ELogVerbosity::Error);
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogSlateStyle, ELogVerbosity::Error);
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogUObjectGlobals, ELogVerbosity::Error);

	InitAll(true, true);
}

GROUP_AFTER_GLOBAL(Catch::DefaultGroup)
{
	CleanupAll();
	GIsRunning = false;
}