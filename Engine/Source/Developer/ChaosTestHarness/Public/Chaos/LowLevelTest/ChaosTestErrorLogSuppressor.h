// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "Misc/FeedbackContext.h"

namespace Chaos::LowLevelTest
{
	/**
	 * RAII guard that suppresses Error and Warning verbosity log messages matching registered substrings.
	 * Replaces GWarn and adds itself to GLog so that matching lines are swallowed rather than forwarded
	 * to output devices that Horde scans. Non-matching messages are forwarded normally so unexpected
	 * errors/warnings still surface.
	 *
	 * Usage:
	 *   FErrorLogSuppressor Suppressor;
	 *   Suppressor.ExpectError(TEXT("expected error substring"));
	 *   Suppressor.ExpectWarning(TEXT("expected warning substring"));
	 *   // ... code that logs the expected messages ...
	 */
	class UE_INTERNAL FErrorLogSuppressor : public FFeedbackContext
	{
	public:
		UE_INTERNAL CHAOSTESTHARNESS_API FErrorLogSuppressor();
		UE_INTERNAL CHAOSTESTHARNESS_API ~FErrorLogSuppressor();
		FErrorLogSuppressor(const FErrorLogSuppressor&) = delete;
		FErrorLogSuppressor(FErrorLogSuppressor&&) = delete;
		FErrorLogSuppressor& operator=(const FErrorLogSuppressor&) = delete;
		FErrorLogSuppressor& operator=(FErrorLogSuppressor&&) = delete;

		// Register a substring pattern. Errors containing this text will be swallowed.
		UE_INTERNAL CHAOSTESTHARNESS_API FErrorLogSuppressor& ExpectError(FStringView InPattern);

		// Register a substring pattern. Warnings containing this text will be swallowed.
		UE_INTERNAL CHAOSTESTHARNESS_API FErrorLogSuppressor& ExpectWarning(FStringView InPattern);

		virtual void Serialize(const TCHAR* InData, ELogVerbosity::Type Verbosity, const FName& Category) override;

	private:
		TArray<FString> ErrorPatterns;
		TArray<FString> WarningPatterns;
		FFeedbackContext* OldWarn;
	};
} // namespace Chaos::LowLevelTest
