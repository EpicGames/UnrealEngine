// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Chaos/ImplicitFwd.h"
#include "Chaos/SimCallbackObject.h"

class FChaosScene;

namespace Chaos
{
	class FPBDRigidsSolver;
	class FSingleParticlePhysicsProxy;

	struct FCollisionEventData;
}

namespace Chaos::LowLevelTest
{
	//
	// A wrapper around a chaos solver providing minimal functionality for testing.
	//
	class UE_INTERNAL FChaosTestScene
	{
	public:
		UE_INTERNAL CHAOSTESTHARNESS_API FChaosTestScene();
		UE_INTERNAL CHAOSTESTHARNESS_API ~FChaosTestScene();

		UE_INTERNAL CHAOSTESTHARNESS_API Chaos::FPBDRigidsSolver* GetSolver();

		// Tick the solver.
		UE_INTERNAL CHAOSTESTHARNESS_API void TickSolver(float Dt);

		// Tick the solver until it sleeps (or until we reach MaxTicks ticks).
		// Return true if the scene is sleeping.
		UE_INTERNAL CHAOSTESTHARNESS_API bool TickSolverToSleep(float Dt, int32 MaxTicks);

		// Return true if all dynamic bodies are asleep.
		UE_INTERNAL CHAOSTESTHARNESS_API bool IsSceneSleeping() const;

		// How many ticks have been run (useful if using TickSolverToSleep)
		UE_INTERNAL CHAOSTESTHARNESS_API int GetNumTicks() const;

		// Add a dynamic awake body to the scene.
		UE_INTERNAL CHAOSTESTHARNESS_API Chaos::FSingleParticlePhysicsProxy* AddDynamicSphere(const TCHAR* InName, const FTransform3d& InTransform, const double& InRadius);
		UE_INTERNAL CHAOSTESTHARNESS_API Chaos::FSingleParticlePhysicsProxy* AddDynamicBox(const TCHAR* InName, const FTransform3d& InTransform, const FVector3d& InHalfExtents);
		UE_INTERNAL CHAOSTESTHARNESS_API Chaos::FSingleParticlePhysicsProxy* AddDynamicProxy(const TCHAR* InName, const FTransform3d& InTransform, const Chaos::FImplicitObjectPtr& InGeometry);

		// Add a kinematic (which can be promoted to dynamic) body to the scene
		UE_INTERNAL CHAOSTESTHARNESS_API Chaos::FSingleParticlePhysicsProxy* AddKinematicBox(const TCHAR* InName, const FTransform3d& InTransform, const FVector3d& InHalfExtents);
		UE_INTERNAL CHAOSTESTHARNESS_API Chaos::FSingleParticlePhysicsProxy* AddKinematicProxy(const TCHAR* InName, const FTransform3d& InTransform, const Chaos::FImplicitObjectPtr& InGeometry);

		// Set the gravity (for gravity group 0)
		UE_INTERNAL CHAOSTESTHARNESS_API void SetGravity(const FVector3d& InG);

		// Set the collision event callback (can only be one)
		UE_INTERNAL CHAOSTESTHARNESS_API void SetCollisionCallback(const TFunction<void(const Chaos::FCollisionEventData&)>& InCollisionCallback);

	private:
		void CreateScene();
		void DestroyScene();
		void CreateSolver();
		void DestroySolver();

		Chaos::FSingleParticlePhysicsProxy* CreateDynamicSphere(const TCHAR* InName, const FTransform3d& InTransform, const double& InRadius);
		Chaos::FSingleParticlePhysicsProxy* CreateDynamicBox(const TCHAR* InName, const FTransform3d& InTransform, const FVector3d& InHalfExtents);
		Chaos::FSingleParticlePhysicsProxy* CreateDynamicProxy(const TCHAR* InName, const FTransform3d& InTransform, const Chaos::FImplicitObjectPtr& InGeometry);
		void InitCollisionFilter(Chaos::FSingleParticlePhysicsProxy* Proxy);
		void InitMassProperties(Chaos::FSingleParticlePhysicsProxy* Proxy);

		void OnCollision(const Chaos::FCollisionEventData& CollisionEventData);

	private:
		FChaosScene* Scene = nullptr;
		Chaos::FPBDRigidsSolver* Solver;
		TArray<Chaos::FSingleParticlePhysicsProxy*> Proxies;
		TFunction<void(const Chaos::FCollisionEventData&)> CollisionCallback;
		int32 NumTicks;
	};

	// A sim callback that allows unit tests to hook a lambda up to the various callbacks
	// TODO: Add all the other callbacks!
	class FChaosTestSimCallback : public Chaos::TSimCallbackObject<Chaos::FSimCallbackNoInput, Chaos::FSimCallbackNoOutput, Chaos::ESimCallbackOptions::ContactModification>
	{
	public:
		virtual void OnContactModification_Internal(Chaos::FCollisionContactModifier& Modifier) override final
		{
			if (ContactModifier)
			{
				ContactModifier(Modifier);
			}
		}

		TFunction<void(Chaos::FCollisionContactModifier&)> ContactModifier;
	};

}