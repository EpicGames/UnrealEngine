// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/AABB.h"
#include "Chaos/ChaosLogUtil.h"
#include "Chaos/LowLevelTest/ChaosTestCatch2.h"

#include <string>

namespace Catch
{
	template<typename T>
	struct StringMaker<UE::Math::TQuat<T>>
	{
		static std::string convert(UE::Math::TQuat<T> const& Value) { return StringMaker<FString>::convert(Value.ToString()); }
	};

	template<>
	struct StringMaker<Chaos::FRotation3>
	{
		static std::string convert(Chaos::FRotation3 const& Value) { return std::string(TCHAR_TO_UTF8(*Chaos::ToString(Value))); }
	};

	template<>
	struct StringMaker<Chaos::FRotation3f>
	{
		static std::string convert(Chaos::FRotation3f const& Value) { return std::string(TCHAR_TO_UTF8(*Chaos::ToString(Value))); }
	};

	template<>
	struct StringMaker<Chaos::FVec3>
	{
		static std::string convert(Chaos::FVec3 const& Value) { return std::string(TCHAR_TO_UTF8(*Chaos::ToString(Value))); }
	};

	template<>
	struct StringMaker<Chaos::FVec3f>
	{
		static std::string convert(Chaos::FVec3f const& Value) { return std::string(TCHAR_TO_UTF8(*Chaos::ToString(Value))); }
	};

	template<typename T>
	struct StringMaker<UE::Math::TVector<T>>
	{
		static std::string convert(UE::Math::TVector<T> const& Value) { return StringMaker<Chaos::TVec3<T>>::convert(Chaos::TVec3<T>(Value)); }
	};

	template<typename T, int d>
	struct StringMaker<Chaos::TAABB<T, d>>
	{
		static std::string convert(Chaos::TAABB<T, d> const& Value) { return StringMaker<FString>::convert(Value.ToString()); }
	};
}